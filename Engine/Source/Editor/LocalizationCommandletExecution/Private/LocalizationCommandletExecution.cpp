// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationCommandletExecution.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Widgets/SNullWidget.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "UnrealEdMisc.h"
#include "LocalizationSettings.h"
#include "LocalizationConfigurationScript.h"
#include "DesktopPlatformModule.h"
#include "Widgets/Images/SThrobber.h"
#include "Commandlets/CommandletHelpers.h"
#include "SourceControlHelpers.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "LocalizationCommandletExecutor"

namespace
{
	class SLocalizationCommandletExecutor : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SLocalizationCommandletExecutor) {}
		SLATE_END_ARGS()

	private:
		struct FTaskListModel
		{
			enum class EState
			{
				Queued,
				InProgress,
				Failed,
				Succeeded
			};

			FTaskListModel()
				: State(EState::Queued)
			{
			}

			LocalizationCommandletExecution::FTask Task;
			EState State;
			FString LogOutput;
			FString ProcessArguments;
		};

		friend class STaskRow;

	public:
		SLocalizationCommandletExecutor();
		~SLocalizationCommandletExecutor();

		void Construct(const FArguments& Arguments, const TSharedRef<SWindow>& ParentWindow, const TArray<LocalizationCommandletExecution::FTask>& Tasks);
		void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
		bool WasSuccessful() const;
		void Log(const FString& String);

	private:
		void ExecuteCommandlet(const TSharedRef<FTaskListModel>& TaskListModel);
		void OnCommandletProcessCompletion(const int32 ReturnCode);
		void CancelCommandlet();
		void CleanUpProcessAndPump();

		bool HasCompleted() const;

		FText GetProgressMessageText() const;
		TOptional<float> GetProgressPercentage() const;

		TSharedRef<ITableRow> OnGenerateTaskListRow(TSharedPtr<FTaskListModel> TaskListModel, const TSharedRef<STableViewBase>& Table);
		TSharedPtr<FTaskListModel> GetCurrentTaskToView() const;

		FText GetCurrentTaskProcessArguments() const;
		FText GetLogString() const;

		FReply OnCopyLogClicked();
		void CopyLogToClipboard();

		FReply OnSaveLogClicked();

		FText GetCloseButtonText() const;
		FReply OnCloseButtonClicked();

	private:
		int32 CurrentTaskIndex;
		TArray< TSharedPtr<FTaskListModel> > TaskListModels;
		TSharedPtr<SProgressBar> ProgressBar;
		TSharedPtr< SListView< TSharedPtr<FTaskListModel> > > TaskListView;

		struct
		{
			FCriticalSection CriticalSection;
			FString String;
		} PendingLogData;


		TSharedPtr<SWindow> ParentWindow;
		TSharedPtr<FLocalizationCommandletProcess> CommandletProcess;
		FRunnable* Runnable;
		FRunnableThread* RunnableThread;
	};

	SLocalizationCommandletExecutor::SLocalizationCommandletExecutor()
		: CurrentTaskIndex(INDEX_NONE)
		, Runnable(nullptr)
		, RunnableThread(nullptr)
	{
	}

	SLocalizationCommandletExecutor::~SLocalizationCommandletExecutor()
	{
		CancelCommandlet();
	}

	void SLocalizationCommandletExecutor::Construct(const FArguments& Arguments, const TSharedRef<SWindow>& InParentWindow, const TArray<LocalizationCommandletExecution::FTask>& Tasks)
	{
		ParentWindow = InParentWindow;

		for (const LocalizationCommandletExecution::FTask& Task : Tasks)
		{
			const TSharedRef<FTaskListModel> Model = MakeShareable(new FTaskListModel());
			Model->Task = Task;
			TaskListModels.Add(Model);
		}

		TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
			.Orientation(Orient_Vertical);
		TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
			.Orientation(Orient_Horizontal);

		ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8.0, 16.0, 16.0, 0.0)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(this, &SLocalizationCommandletExecutor::GetProgressMessageText)
					]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0, 4.0, 0.0, 0.0)
						[
							SAssignNew(ProgressBar, SProgressBar)
							.Percent(this, &SLocalizationCommandletExecutor::GetProgressPercentage)
						]
				]
				+ SVerticalBox::Slot()
					.FillHeight(0.5)
					.Padding(0.0, 32.0, 8.0, 0.0)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						.Padding(0.0f)
						[
							SAssignNew(TaskListView, SListView< TSharedPtr<FTaskListModel> >)
							.HeaderRow
							(
							SNew(SHeaderRow)
							+ SHeaderRow::Column("StatusIcon")
							.DefaultLabel(FText::GetEmpty())
							.FixedWidth(20.0)
							+ SHeaderRow::Column("TaskName")
							.DefaultLabel(LOCTEXT("TaskListNameColumnLabel", "Task"))
							.FillWidth(1.0)
							)
							.ListItemsSource(&TaskListModels)
							.OnGenerateRow(this, &SLocalizationCommandletExecutor::OnGenerateTaskListRow)
							.SelectionMode(ESelectionMode::Single)
						]
					]
				+ SVerticalBox::Slot()
					.FillHeight(0.5)
					.Padding(0.0, 32.0, 8.0, 0.0)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						.Padding(0.0f)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
								.FillHeight(1.0f)
								[
									SNew(SMultiLineEditableText)
									.TextStyle(FAppStyle::Get(), "LocalizationDashboard.CommandletLog.Text")
									.Text(this, &SLocalizationCommandletExecutor::GetLogString)
									.IsReadOnly(true)
									.HScrollBar(HorizontalScrollBar)
									.VScrollBar(VerticalScrollBar)
								]
								+SVerticalBox::Slot()
									.AutoHeight()
									[
										HorizontalScrollBar
									]
							]
							+SHorizontalBox::Slot()
								.AutoWidth()
								[
									VerticalScrollBar
								]
						]
					]
				+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 5.0f, 0.0f, 0.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SButton)
							.ContentPadding(FMargin(6.0f, 2.0f))
							.Text(LOCTEXT("CopyLogButtonText", "Copy Log"))
							.ToolTipText(LOCTEXT("CopyLogButtonTooltip", "Copy the logged text to the clipboard."))
							.OnClicked(this, &SLocalizationCommandletExecutor::OnCopyLogClicked)
						]
						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.ContentPadding(FMargin(6.0f, 2.0f))
								.Text(LOCTEXT("SaveLogButtonText", "Save Log..."))
								.ToolTipText(LOCTEXT("SaveLogButtonToolTip", "Save the logged text to a file."))
								.OnClicked(this, &SLocalizationCommandletExecutor::OnSaveLogClicked)
							]
						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.ContentPadding(FMargin(6.0f, 2.0f))
								.OnClicked(this, &SLocalizationCommandletExecutor::OnCloseButtonClicked)
								[
									SNew(STextBlock)
									.Text(this, &SLocalizationCommandletExecutor::GetCloseButtonText)
								]
							]
					]
			];

		if(TaskListModels.Num() > 0)
		{
			CurrentTaskIndex = 0;
			ExecuteCommandlet(TaskListModels[CurrentTaskIndex].ToSharedRef());

			if (TaskListView.IsValid())
			{
				TaskListView->SetSelection(TaskListModels[CurrentTaskIndex]);
			}
		}
	}

	void SLocalizationCommandletExecutor::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		// Poll for log output data.
		if (!PendingLogData.String.IsEmpty())
		{
			FString String;

			// Copy the pending data string to the local string
			{
				FScopeLock ScopeLock(&PendingLogData.CriticalSection);
				String = PendingLogData.String;
				PendingLogData.String.Empty();
			}

			// Forward string to proper log.
			if (TaskListModels.IsValidIndex(CurrentTaskIndex))
			{
				const TSharedPtr<FTaskListModel> CurrentTaskModel = TaskListModels[CurrentTaskIndex];
				CurrentTaskModel->LogOutput.Append(String);
			}
		}

		// On Task Completed.
		if (CommandletProcess.IsValid())
		{
			FProcHandle CurrentProcessHandle = CommandletProcess->GetHandle();
			int32 ReturnCode;
			if (CurrentProcessHandle.IsValid() && FPlatformProcess::GetProcReturnCode(CurrentProcessHandle, &ReturnCode))
			{
				OnCommandletProcessCompletion(ReturnCode);
			}
		}
	}

	bool SLocalizationCommandletExecutor::WasSuccessful() const
	{
		return HasCompleted() && !TaskListModels.ContainsByPredicate([](const TSharedPtr<FTaskListModel>& TaskListModel){return TaskListModel->State != FTaskListModel::EState::Succeeded;});
	}

	void SLocalizationCommandletExecutor::Log(const FString& String)
	{
		FScopeLock ScopeLock(&PendingLogData.CriticalSection);
		PendingLogData.String += String;
	}

	void SLocalizationCommandletExecutor::OnCommandletProcessCompletion(const int32 ReturnCode)
	{
		CleanUpProcessAndPump();

		// Handle return code.
		TSharedPtr<FTaskListModel> CurrentTaskModel = TaskListModels[CurrentTaskIndex];

		// Restore engine's source control settings if necessary.
		if (!CurrentTaskModel->Task.ShouldUseProjectFile)
		{
			const FString& EngineIniFile = SourceControlHelpers::GetGlobalSettingsIni();
			const FString BackupEngineIniFile = FPaths::EngineSavedDir() / FPaths::GetCleanFilename(EngineIniFile) + TEXT(".bak");
			if(!IFileManager::Get().Move(*EngineIniFile, *BackupEngineIniFile))
			{
				// TODO: Error failed to restore engine source control settings INI.
			}
		}

		// Zero code is successful.
		if (ReturnCode == 0)
		{
			CurrentTaskModel->State = FTaskListModel::EState::Succeeded;

			++CurrentTaskIndex;

			// Begin new task if possible.
			if (TaskListModels.IsValidIndex(CurrentTaskIndex))
			{
				CurrentTaskModel = TaskListModels[CurrentTaskIndex];
				ExecuteCommandlet(CurrentTaskModel.ToSharedRef());

				if (TaskListView.IsValid())
				{
					TaskListView->SetSelection(TaskListModels[CurrentTaskIndex]);
				}
			}
		}
		// Non-zero is a failure.
		else
		{
			CurrentTaskModel->State = FTaskListModel::EState::Failed;
		}
	}

	void SLocalizationCommandletExecutor::ExecuteCommandlet(const TSharedRef<FTaskListModel>& TaskListModel)
	{
		// Handle source control settings if not using project file for commandlet executable process.
		if (!TaskListModel->Task.ShouldUseProjectFile)
		{
			const FString& EngineIniFile = SourceControlHelpers::GetGlobalSettingsIni();

			// Backup engine's source control settings.
			const FString BackupEngineIniFile = FPaths::EngineSavedDir() / FPaths::GetCleanFilename(EngineIniFile) + TEXT(".bak");
			if (COPY_OK == IFileManager::Get().Copy(*BackupEngineIniFile, *EngineIniFile))
			{
				// Replace engine's source control settings with project's.
				const FString& ProjectIniFile = SourceControlHelpers::GetSettingsIni();
				if (COPY_OK == IFileManager::Get().Copy(*EngineIniFile, *ProjectIniFile))
				{
					// TODO: Error failed to overwrite engine source control settings INI.
				}
			}
			else
			{
				// TODO: Error failed to backup engine source control settings INI.
			}
		}

		CommandletProcess = FLocalizationCommandletProcess::Execute(TaskListModel->Task.ScriptPath, TaskListModel->Task.ShouldUseProjectFile);
		if (CommandletProcess.IsValid())
		{
			TaskListModel->State = FTaskListModel::EState::InProgress;
			TaskListModel->ProcessArguments = CommandletProcess->GetProcessArguments();
		}
		else
		{
			TaskListModel->State = FTaskListModel::EState::Failed;
			return;
		}

		class FCommandletLogPump : public FRunnable
		{
		public:
			FCommandletLogPump(void* const InReadPipe, const FProcHandle& InCommandletProcessHandle, SLocalizationCommandletExecutor& InCommandletWidget)
				: ReadPipe(InReadPipe)
				, CommandletProcessHandle(InCommandletProcessHandle)
				, CommandletWidget(&InCommandletWidget)
			{
			}

			uint32 Run() override
			{
				for(;;)
				{
					// Read from pipe.
					const FString PipeString = FPlatformProcess::ReadPipe(ReadPipe);

					// Process buffer.
					if (!PipeString.IsEmpty())
					{
						// Add strings to log.
						if (CommandletWidget)
						{
							CommandletWidget->Log(PipeString);
						}
					}

					// If the process isn't running and there's no data in the pipe, we're done.
					if (!FPlatformProcess::IsProcRunning(CommandletProcessHandle) && PipeString.IsEmpty())
					{
						break;
					}

					// Sleep.
					FPlatformProcess::Sleep(0.0f);
				}

				int32 ReturnCode = 0;
				return FPlatformProcess::GetProcReturnCode(CommandletProcessHandle, &ReturnCode) ? ReturnCode : -1;
			}

		private:
			void* const ReadPipe;
			FProcHandle CommandletProcessHandle;
			SLocalizationCommandletExecutor* const CommandletWidget;
		};

		// Launch runnable thread.
		Runnable = new FCommandletLogPump(CommandletProcess->GetReadPipe(), CommandletProcess->GetHandle(), *this);
		RunnableThread = FRunnableThread::Create(Runnable, TEXT("Localization Commandlet Log Pump Thread"));
	}

	void SLocalizationCommandletExecutor::CancelCommandlet()
	{
		CleanUpProcessAndPump();
	}

	void SLocalizationCommandletExecutor::CleanUpProcessAndPump()
	{
		if (CommandletProcess.IsValid())
		{
			FProcHandle CommandletProcessHandle = CommandletProcess->GetHandle();
			if (CommandletProcessHandle.IsValid() && FPlatformProcess::IsProcRunning(CommandletProcessHandle))
			{
				FPlatformProcess::TerminateProc(CommandletProcessHandle, true);
			}
		}

		if (RunnableThread)
		{
			RunnableThread->WaitForCompletion();
			delete RunnableThread;
			RunnableThread = nullptr;
		}

		if (Runnable)
		{
			delete Runnable;
			Runnable = nullptr;
		}

		// Reset now to close the read and write pipes which were used by the FCommandletLogPump thread
		CommandletProcess.Reset();
	}

	bool SLocalizationCommandletExecutor::HasCompleted() const
	{
		return CurrentTaskIndex == TaskListModels.Num();
	}

	FText SLocalizationCommandletExecutor::GetProgressMessageText() const
	{
		return TaskListModels.IsValidIndex(CurrentTaskIndex) ? TaskListModels[CurrentTaskIndex]->Task.Name : FText::GetEmpty();
	}

	TOptional<float> SLocalizationCommandletExecutor::GetProgressPercentage() const
	{
		return TOptional<float>(float(CurrentTaskIndex) / float(TaskListModels.Num()));
	}

	class STaskRow : public SMultiColumnTableRow< TSharedPtr<SLocalizationCommandletExecutor::FTaskListModel> >
	{
	public:
		void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<SLocalizationCommandletExecutor::FTaskListModel>& InTaskListModel);
		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName);

	private:
		FSlateColor HandleIconColorAndOpacity() const;
		const FSlateBrush* HandleIconImage() const;
		EVisibility HandleThrobberVisibility() const;

	private:
		TSharedPtr<SLocalizationCommandletExecutor::FTaskListModel> TaskListModel;
	};

	void STaskRow::Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<SLocalizationCommandletExecutor::FTaskListModel>& InTaskListModel)
	{
		TaskListModel = InTaskListModel;

		FSuperRowType::Construct(InArgs, OwnerTableView);
	}

	TSharedRef<SWidget> STaskRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		if (ColumnName == "StatusIcon")
		{
			return SNew(SOverlay)

				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SThrobber)
					.Animate(SThrobber::VerticalAndOpacity)
					.NumPieces(1)
					.Visibility(this, &STaskRow::HandleThrobberVisibility)
				]
			+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.ColorAndOpacity(this, &STaskRow::HandleIconColorAndOpacity)
					.Image(this, &STaskRow::HandleIconImage)
				];
		}
		else if (ColumnName == "TaskName")
		{
			return SNew(STextBlock)
				.Text(TaskListModel->Task.Name)
				.ToolTipText_Lambda( [this]{ return FText::FromString(TaskListModel->ProcessArguments); } );
		}
		else
		{
			return SNullWidget::NullWidget;
		}
	}

	FSlateColor STaskRow::HandleIconColorAndOpacity( ) const
	{
		if (TaskListModel.IsValid())
		{
			switch(TaskListModel->State)
			{
			case SLocalizationCommandletExecutor::FTaskListModel::EState::InProgress:
				return FLinearColor::Yellow;
			case SLocalizationCommandletExecutor::FTaskListModel::EState::Succeeded:
				return FLinearColor::Green;
			case SLocalizationCommandletExecutor::FTaskListModel::EState::Failed:
				return FLinearColor::Red;
			}
		}

		return FSlateColor::UseForeground();
	}

	const FSlateBrush* STaskRow::HandleIconImage( ) const
	{
		if (TaskListModel.IsValid())
		{
			switch(TaskListModel->State)
			{
			case SLocalizationCommandletExecutor::FTaskListModel::EState::Succeeded:
				return FAppStyle::GetBrush("Symbols.Check");
			case SLocalizationCommandletExecutor::FTaskListModel::EState::Failed:
				return FAppStyle::GetBrush("Icons.Cross");
			}
		}

		return NULL;
	}

	EVisibility STaskRow::HandleThrobberVisibility( ) const
	{
		if (TaskListModel.IsValid())
		{
			switch(TaskListModel->State)
			{
			case SLocalizationCommandletExecutor::FTaskListModel::EState::InProgress:
				return EVisibility::Visible;
			}
		}

		return EVisibility::Hidden;
	}

	TSharedRef<ITableRow> SLocalizationCommandletExecutor::OnGenerateTaskListRow(TSharedPtr<FTaskListModel> TaskListModel, const TSharedRef<STableViewBase>& Table)
	{
		return SNew(STaskRow, Table, TaskListModel.ToSharedRef());
	}

	TSharedPtr<SLocalizationCommandletExecutor::FTaskListModel> SLocalizationCommandletExecutor::GetCurrentTaskToView() const
	{
		if (TaskListView.IsValid())
		{
			const TArray< TSharedPtr<FTaskListModel> > Selection = TaskListView->GetSelectedItems();
			return Selection.Num() > 0 ? Selection.Top() : nullptr;
		}
		return nullptr;
	}

	FText SLocalizationCommandletExecutor::GetCurrentTaskProcessArguments() const
	{
		const TSharedPtr<SLocalizationCommandletExecutor::FTaskListModel> TaskToView = GetCurrentTaskToView();
		return TaskToView.IsValid() ? FText::FromString(TaskToView->ProcessArguments) : FText::GetEmpty();
	}

	FText SLocalizationCommandletExecutor::GetLogString() const
	{
		const TSharedPtr<SLocalizationCommandletExecutor::FTaskListModel> TaskToView = GetCurrentTaskToView();
		return TaskToView.IsValid() ? FText::FromString(TaskToView->LogOutput) : FText::GetEmpty();
	}

	FReply SLocalizationCommandletExecutor::OnCopyLogClicked()
	{
		CopyLogToClipboard();

		return FReply::Handled();
	}

	void SLocalizationCommandletExecutor::CopyLogToClipboard()
	{
		FPlatformApplicationMisc::ClipboardCopy(*(GetLogString().ToString()));
	}

	FReply SLocalizationCommandletExecutor::OnSaveLogClicked()
	{
		const FString TextFileDescription = LOCTEXT("TextFileDescription", "Text File").ToString();
		const FString TextFileExtension = TEXT("txt");
		const FString TextFileExtensionWildcard = FString::Printf(TEXT("*.%s"), *TextFileExtension);
		const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *TextFileDescription, *TextFileExtensionWildcard, *TextFileExtensionWildcard);
		const FString DefaultFilename = FString::Printf(TEXT("%s.%s"), TEXT("Log"), *TextFileExtension);
		const FString DefaultPath = FPaths::ProjectSavedDir();

		TArray<FString> SaveFilenames;
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		// Prompt the user for the filename
		if (DesktopPlatform)
		{
			void* ParentWindowWindowHandle = NULL;

			const TSharedPtr<SWindow>& ParentWindowPtr = FSlateApplication::Get().FindWidgetWindow(AsShared());
			if (ParentWindowPtr.IsValid() && ParentWindowPtr->GetNativeWindow().IsValid())
			{
				ParentWindowWindowHandle = ParentWindowPtr->GetNativeWindow()->GetOSWindowHandle();
			}

			if (DesktopPlatform->SaveFileDialog(
				ParentWindowWindowHandle,
				LOCTEXT("SaveLogDialogTitle", "Save Log to File").ToString(),
				DefaultPath,
				DefaultFilename,
				FileTypes,
				EFileDialogFlags::None,
				SaveFilenames
				))
			{
				// Save to file.
				FFileHelper::SaveStringToFile( GetLogString().ToString(), *(SaveFilenames.Last()) );
			}
		}

		return FReply::Handled();
	}

	FText SLocalizationCommandletExecutor::GetCloseButtonText() const
	{
		return HasCompleted() ? LOCTEXT("OkButtonText", "OK") : LOCTEXT("CancelButtonText", "Cancel");
	}

	FReply SLocalizationCommandletExecutor::OnCloseButtonClicked()
	{
		if (!HasCompleted())
		{
			CancelCommandlet();
		}
		ParentWindow->RequestDestroyWindow();
		return FReply::Handled();
	}
}

bool LocalizationCommandletExecution::Execute(const TSharedRef<SWindow>& ParentWindow, const FText& Title, const TArray<FTask>& Tasks)
{
	const TSharedRef<SWindow> CommandletWindow = SNew(SWindow)
		.Title(Title)
		.SupportsMinimize(false)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.ClientSize(FVector2D(600,400))
		.ActivationPolicy(EWindowActivationPolicy::Always)
		.FocusWhenFirstShown(true);
	const TSharedRef<SLocalizationCommandletExecutor> CommandletExecutor = SNew(SLocalizationCommandletExecutor, CommandletWindow, Tasks);
	CommandletWindow->SetContent(CommandletExecutor);

	FSlateApplication::Get().AddModalWindow(CommandletWindow, ParentWindow, false);
	return CommandletExecutor->WasSuccessful();
}

TSharedPtr<FLocalizationCommandletProcess> FLocalizationCommandletProcess::Execute(const FString& ConfigFilePath, const bool UseProjectFile)
{
	// Create pipes.
	void* ReadPipe;
	void* WritePipe;
	if (!FPlatformProcess::CreatePipe(ReadPipe, WritePipe))
	{
		return nullptr;
	}

	// Create process.
	FString CommandletArguments;

	const FString ConfigFileRelativeToGameDir = LocalizationConfigurationScript::MakePathRelativeForCommandletProcess(ConfigFilePath, UseProjectFile);
	CommandletArguments = FString::Printf( TEXT("-config=\"%s\""), *ConfigFileRelativeToGameDir );

	if (FLocalizationSourceControlSettings::IsSourceControlEnabled())
	{
		CommandletArguments += TEXT(" -EnableSCC");

		if (!FLocalizationSourceControlSettings::IsSourceControlAutoSubmitEnabled())
		{
			CommandletArguments += TEXT(" -DisableSCCSubmit");
		}
	}

	CommandletArguments += TEXT(" -Unattended");

	const FString ProjectFilePath = FString::Printf(TEXT("\"%s\""), *FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath()));
	const FString ProcessArguments = CommandletHelpers::BuildCommandletProcessArguments(TEXT("GatherText"), UseProjectFile ? *ProjectFilePath : nullptr, *CommandletArguments);
	FProcHandle CommandletProcessHandle = FPlatformProcess::CreateProc(*FUnrealEdMisc::Get().GetExecutableForCommandlets(), *ProcessArguments, true, true, true, nullptr, 0, nullptr, WritePipe);

	// Close pipes if process failed.
	if (!CommandletProcessHandle.IsValid())
	{
		FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
		return nullptr;
	}

	return MakeShareable(new FLocalizationCommandletProcess(ReadPipe, WritePipe, CommandletProcessHandle, ProcessArguments));
}

FLocalizationCommandletProcess::~FLocalizationCommandletProcess()
{
	if (ProcessHandle.IsValid() && FPlatformProcess::IsProcRunning(ProcessHandle))
	{
		FPlatformProcess::TerminateProc(ProcessHandle);
	}
	FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
}

#undef LOCTEXT_NAMESPACE
