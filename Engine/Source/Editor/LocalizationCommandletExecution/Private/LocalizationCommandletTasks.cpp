// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationCommandletTasks.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Internationalization/Culture.h"
#include "UObject/UObjectHash.h"
#include "Sound/SoundWave.h"
#include "Sound/DialogueWave.h"
#include "LocalizationCommandletExecution.h"
#include "LocalizationConfigurationScript.h"
#include "LocalizationDelegates.h"
#include "LocalizationTargetTypes.h"
#include "HAL/FileManager.h"

#define LOCTEXT_NAMESPACE "LocalizationCommandletTasks"

namespace LocalizationCommandletTasks
{

void BroadcastLocalizationTargetsDataUpdated(TArrayView<ULocalizationTarget* const> Targets)
{
	for (ULocalizationTarget* Target : Targets)
	{
		const FString TargetContentPath = FPaths::Combine(Target->IsMemberOfEngineTargetSet() ? FPaths::EngineContentDir() : FPaths::ProjectContentDir(), TEXT("Localization"), Target->Settings.Name);
		LocalizationDelegates::OnLocalizationTargetDataUpdated.Broadcast(TargetContentPath);
	}
}

bool ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(const TSharedRef<SWindow>& ParentWindow, const FText& Title, const TArray<LocalizationCommandletExecution::FTask>& Tasks, TArrayView<ULocalizationTarget* const> Targets)
{
	if (LocalizationCommandletExecution::Execute(ParentWindow, Title, Tasks))
	{
		BroadcastLocalizationTargetsDataUpdated(Targets);
		return true;
	}
	return false;
}

bool ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(const TSharedRef<SWindow>& ParentWindow, const FText& Title, const TArray<LocalizationCommandletExecution::FTask>& Tasks, ULocalizationTarget* const Target)
{
	if (LocalizationCommandletExecution::Execute(ParentWindow, Title, Tasks))
	{
		BroadcastLocalizationTargetsDataUpdated(MakeArrayView(&Target, 1));
		return true;
	}
	return false;
}

}

bool LocalizationCommandletTasks::GatherTextForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;

	for (ULocalizationTarget* Target : Targets)
	{
		const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

		const FText GatherTaskName = FText::Format(LOCTEXT("GatherTaskNameFormat", "Gather Text for {TargetName}"), Arguments);
		const FString GatherScriptPath = LocalizationConfigurationScript::GetGatherTextConfigPath(Target);
		LocalizationConfigurationScript::GenerateGatherTextConfigFile(Target).WriteWithSCC(GatherScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(GatherTaskName, GatherScriptPath, ShouldUseProjectFile));
	}

	return ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(ParentWindow, LOCTEXT("GatherAllTargetsWindowTitle", "Gather Text for All Targets"), Tasks, Targets);
}

bool LocalizationCommandletTasks::GatherTextForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString GatherScriptPath = LocalizationConfigurationScript::GetGatherTextConfigPath(Target);
	LocalizationConfigurationScript::GenerateGatherTextConfigFile(Target).WriteWithSCC(GatherScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("GatherTaskName", "Gather Text"), GatherScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("GatherTargetWindowTitle", "Gather Text for Target {TargetName}"), Arguments);
	return ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
}


bool LocalizationCommandletTasks::ImportTextForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets, const TOptional<FString> DirectoryPath)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;

	for (ULocalizationTarget* Target : Targets)
	{
		const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

		const FText ImportTaskName = FText::Format(LOCTEXT("ImportTaskNameFormat", "Import Translations for {TargetName}"), Arguments);
		const FString ImportScriptPath = LocalizationConfigurationScript::GetImportTextConfigPath(Target, TOptional<FString>());
		const TOptional<FString> DirectoryPathForTarget = DirectoryPath.IsSet() ? DirectoryPath.GetValue() / Target->Settings.Name : TOptional<FString>();
		LocalizationConfigurationScript::GenerateImportTextConfigFile(Target, TOptional<FString>(), DirectoryPathForTarget).WriteWithSCC(ImportScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(ImportTaskName, ImportScriptPath, ShouldUseProjectFile));

		const FText ReportTaskName = FText::Format(LOCTEXT("ReportTaskNameFormat", "Generate Reports for {TargetName}"), Arguments);
		const FString ReportScriptPath = LocalizationConfigurationScript::GetWordCountReportConfigPath(Target);
		LocalizationConfigurationScript::GenerateWordCountReportConfigFile(Target).WriteWithSCC(ReportScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(ReportTaskName, ReportScriptPath, ShouldUseProjectFile));
	}

	return ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(ParentWindow, LOCTEXT("ImportForAllTargetsWindowTitle", "Import Translations for All Targets"), Tasks, Targets);
}

bool LocalizationCommandletTasks::ImportTextForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const TOptional<FString> DirectoryPath)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString ImportScriptPath = LocalizationConfigurationScript::GetImportTextConfigPath(Target, TOptional<FString>());
	LocalizationConfigurationScript::GenerateImportTextConfigFile(Target, TOptional<FString>(), DirectoryPath).WriteWithSCC(ImportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ImportTaskName", "Import Translations"), ImportScriptPath, ShouldUseProjectFile));

	const FString ReportScriptPath = LocalizationConfigurationScript::GetWordCountReportConfigPath(Target);
	LocalizationConfigurationScript::GenerateWordCountReportConfigFile(Target).WriteWithSCC(ReportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ReportTaskName", "Generate Reports"), ReportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ImportForTargetWindowTitle", "Import Translations for Target {TargetName}"), Arguments);
	return ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
}

bool LocalizationCommandletTasks::ImportTextForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName, const TOptional<FString> FilePath)
{
	FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
	if (!Culture.IsValid())
	{
		return false;
	}

	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString DefaultImportScriptPath = LocalizationConfigurationScript::GetImportTextConfigPath(Target, TOptional<FString>(CultureName));
	const FString ImportScriptPath = FPaths::CreateTempFilename(*FPaths::GetPath(DefaultImportScriptPath), *FPaths::GetBaseFilename(DefaultImportScriptPath), *FPaths::GetExtension(DefaultImportScriptPath, true));
	LocalizationConfigurationScript::GenerateImportTextConfigFile(Target, TOptional<FString>(CultureName), FilePath).Write(ImportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ImportTaskName", "Import Translations"), ImportScriptPath, ShouldUseProjectFile));

	const FString ReportScriptPath = LocalizationConfigurationScript::GetWordCountReportConfigPath(Target);
	LocalizationConfigurationScript::GenerateWordCountReportConfigFile(Target).Write(ReportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ReportTaskName", "Generate Reports"), ReportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ImportCultureForTargetWindowTitle", "Import {CultureName} Translations for Target {TargetName}"), Arguments);

	bool HasSucceeeded = ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
	IFileManager::Get().Delete(*ImportScriptPath); // Don't clutter up the loc config directory with scripts for individual cultures.
	return HasSucceeeded;
}

bool LocalizationCommandletTasks::ExportTextForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets, const TOptional<FString> DirectoryPath)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;

	for (ULocalizationTarget* Target : Targets)
	{
		const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

		const FText ExportTaskName = FText::Format(LOCTEXT("ExportTaskNameFormat", "Export Translations for {TargetName}"), Arguments);
		const FString ExportScriptPath = LocalizationConfigurationScript::GetExportTextConfigPath(Target, TOptional<FString>());
		const TOptional<FString> DirectoryPathForTarget = DirectoryPath.IsSet() ? DirectoryPath.GetValue() / Target->Settings.Name : TOptional<FString>();
		LocalizationConfigurationScript::GenerateExportTextConfigFile(Target, TOptional<FString>(), DirectoryPathForTarget).WriteWithSCC(ExportScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(ExportTaskName, ExportScriptPath, ShouldUseProjectFile));
	}

	return ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(ParentWindow, LOCTEXT("ExportForAllTargetsWindowTitle", "Export Translations for All Targets"), Tasks, Targets);
}

bool LocalizationCommandletTasks::ExportTextForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const TOptional<FString> DirectoryPath)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString ExportScriptPath = LocalizationConfigurationScript::GetExportTextConfigPath(Target, TOptional<FString>());
	LocalizationConfigurationScript::GenerateExportTextConfigFile(Target, TOptional<FString>(), DirectoryPath).WriteWithSCC(ExportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ExportTaskName", "Export Translations"), ExportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ExportForTargetWindowTitle", "Export Translations for Target {TargetName}"), Arguments);
	return ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
}

bool LocalizationCommandletTasks::ExportTextForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName, const TOptional<FString> FilePath)
{
	FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
	if (!Culture.IsValid())
	{
		return false;
	}

	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString DefaultExportScriptPath = LocalizationConfigurationScript::GetExportTextConfigPath(Target, TOptional<FString>(CultureName));
	const FString ExportScriptPath = FPaths::CreateTempFilename(*FPaths::GetPath(DefaultExportScriptPath), *FPaths::GetBaseFilename(DefaultExportScriptPath), *FPaths::GetExtension(DefaultExportScriptPath, true));
	LocalizationConfigurationScript::GenerateExportTextConfigFile(Target, TOptional<FString>(CultureName), FilePath).Write(ExportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ExportTaskName", "Export Translations"), ExportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ExportCultureForTargetWindowTitle", "Export {CultureName} Translations for Target {TargetName}"), Arguments);

	bool HasSucceeeded = ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
	IFileManager::Get().Delete(*ExportScriptPath); // Don't clutter up the loc config directory with scripts for individual cultures.
	return HasSucceeeded;
}

bool LocalizationCommandletTasks::ImportDialogueScriptForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets, const TOptional<FString> DirectoryPath)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;

	for (ULocalizationTarget* Target : Targets)
	{
		const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

		const FText ImportTaskName = FText::Format(LOCTEXT("ImportDialogueScriptsTaskNameFormat", "Import Dialogue Scripts for {TargetName}"), Arguments);
		const FString ImportScriptPath = LocalizationConfigurationScript::GetImportDialogueScriptConfigPath(Target, TOptional<FString>());
		const TOptional<FString> DirectoryPathForTarget = DirectoryPath.IsSet() ? DirectoryPath.GetValue() / Target->Settings.Name : TOptional<FString>();
		LocalizationConfigurationScript::GenerateImportDialogueScriptConfigFile(Target, TOptional<FString>(), DirectoryPathForTarget).WriteWithSCC(ImportScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(ImportTaskName, ImportScriptPath, ShouldUseProjectFile));

		const FText ReportTaskName = FText::Format(LOCTEXT("ReportTaskNameFormat", "Generate Reports for {TargetName}"), Arguments);
		const FString ReportScriptPath = LocalizationConfigurationScript::GetWordCountReportConfigPath(Target);
		LocalizationConfigurationScript::GenerateWordCountReportConfigFile(Target).WriteWithSCC(ReportScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(ReportTaskName, ReportScriptPath, ShouldUseProjectFile));
	}

	return ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(ParentWindow, LOCTEXT("ImportDialogueScriptsForAllTargetsWindowTitle", "Import Dialogue Scripts for All Targets"), Tasks, Targets);
}

bool LocalizationCommandletTasks::ImportDialogueScriptForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const TOptional<FString> DirectoryPath)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString ImportScriptPath = LocalizationConfigurationScript::GetImportDialogueScriptConfigPath(Target, TOptional<FString>());
	LocalizationConfigurationScript::GenerateImportDialogueScriptConfigFile(Target, TOptional<FString>(), DirectoryPath).WriteWithSCC(ImportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ImportDialogueScriptsTaskName", "Import Dialogue Scripts"), ImportScriptPath, ShouldUseProjectFile));

	const FString ReportScriptPath = LocalizationConfigurationScript::GetWordCountReportConfigPath(Target);
	LocalizationConfigurationScript::GenerateWordCountReportConfigFile(Target).WriteWithSCC(ReportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ReportTaskName", "Generate Reports"), ReportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ImportDialogueScriptsForTargetWindowTitle", "Import Dialogue Scripts for Target {TargetName}"), Arguments);
	return ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
}

bool LocalizationCommandletTasks::ImportDialogueScriptForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName, const TOptional<FString> FilePath)
{
	FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
	if (!Culture.IsValid())
	{
		return false;
	}

	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString DefaultImportScriptPath = LocalizationConfigurationScript::GetImportDialogueScriptConfigPath(Target, TOptional<FString>(CultureName));
	const FString ImportScriptPath = FPaths::CreateTempFilename(*FPaths::GetPath(DefaultImportScriptPath), *FPaths::GetBaseFilename(DefaultImportScriptPath), *FPaths::GetExtension(DefaultImportScriptPath, true));
	LocalizationConfigurationScript::GenerateImportDialogueScriptConfigFile(Target, TOptional<FString>(CultureName), FilePath).Write(ImportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ImportDialogueScriptsTaskName", "Import Dialogue Scripts"), ImportScriptPath, ShouldUseProjectFile));

	const FString ReportScriptPath = LocalizationConfigurationScript::GetWordCountReportConfigPath(Target);
	LocalizationConfigurationScript::GenerateWordCountReportConfigFile(Target).Write(ReportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ReportTaskName", "Generate Reports"), ReportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ImportDialogueScriptsForCultureForTargetWindowTitle", "Import {CultureName} Dialogue Scripts for Target {TargetName}"), Arguments);

	bool HasSucceeeded = ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
	IFileManager::Get().Delete(*ImportScriptPath); // Don't clutter up the loc config directory with scripts for individual cultures.
	return HasSucceeeded;
}

bool LocalizationCommandletTasks::ExportDialogueScriptForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets, const TOptional<FString> DirectoryPath)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;

	for (ULocalizationTarget* Target : Targets)
	{
		const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

		const FText ExportTaskName = FText::Format(LOCTEXT("ExportDialogueScriptsTaskNameFormat", "Export Dialogue Scripts for {TargetName}"), Arguments);
		const FString ExportScriptPath = LocalizationConfigurationScript::GetExportDialogueScriptConfigPath(Target, TOptional<FString>());
		const TOptional<FString> DirectoryPathForTarget = DirectoryPath.IsSet() ? DirectoryPath.GetValue() / Target->Settings.Name : TOptional<FString>();
		LocalizationConfigurationScript::GenerateExportDialogueScriptConfigFile(Target, TOptional<FString>(), DirectoryPathForTarget).WriteWithSCC(ExportScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(ExportTaskName, ExportScriptPath, ShouldUseProjectFile));
	}

	return ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(ParentWindow, LOCTEXT("ExportDialogueScriptsForAllTargetsWindowTitle", "Export Dialogue Scripts for All Targets"), Tasks, Targets);
}

bool LocalizationCommandletTasks::ExportDialogueScriptForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const TOptional<FString> DirectoryPath)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString ExportScriptPath = LocalizationConfigurationScript::GetExportDialogueScriptConfigPath(Target, TOptional<FString>());
	LocalizationConfigurationScript::GenerateExportDialogueScriptConfigFile(Target, TOptional<FString>(), DirectoryPath).WriteWithSCC(ExportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ExportDialogueScriptsTaskName", "Export Dialogue Scripts"), ExportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ExportDialogueScriptsForTargetWindowTitle", "Export Dialogue Scripts for Target {TargetName}"), Arguments);
	return ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
}

bool LocalizationCommandletTasks::ExportDialogueScriptForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName, const TOptional<FString> FilePath)
{
	FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
	if (!Culture.IsValid())
	{
		return false;
	}

	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString DefaultExportScriptPath = LocalizationConfigurationScript::GetExportDialogueScriptConfigPath(Target, TOptional<FString>(CultureName));
	const FString ExportScriptPath = FPaths::CreateTempFilename(*FPaths::GetPath(DefaultExportScriptPath), *FPaths::GetBaseFilename(DefaultExportScriptPath), *FPaths::GetExtension(DefaultExportScriptPath, true));
	LocalizationConfigurationScript::GenerateExportDialogueScriptConfigFile(Target, TOptional<FString>(CultureName), FilePath).Write(ExportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ExportDialogueScriptsTaskName", "Export Dialogue Scripts"), ExportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ExportDialogueScriptsForCultureForTargetWindowTitle", "Export {CultureName} Dialogue Scripts for Target {TargetName}"), Arguments);

	bool HasSucceeeded = ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
	IFileManager::Get().Delete(*ExportScriptPath); // Don't clutter up the loc config directory with scripts for individual cultures.
	return HasSucceeeded;
}

bool LocalizationCommandletTasks::ImportDialogueForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;

	for (ULocalizationTarget* Target : Targets)
	{
		const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

		const FText ImportDialogueTaskName = FText::Format(LOCTEXT("ImportDialogueTaskNameFormat", "Import Dialogue for {TargetName}"), Arguments);
		const FString ImportDialogueScriptPath = LocalizationConfigurationScript::GetImportDialogueConfigPath(Target);
		LocalizationConfigurationScript::GenerateImportDialogueConfigFile(Target).WriteWithSCC(ImportDialogueScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(ImportDialogueTaskName, ImportDialogueScriptPath, ShouldUseProjectFile));
	}

	return ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(ParentWindow, LOCTEXT("ImportDialogueForAllTargetsWindowTitle", "Import Dialogue for All Targets"), Tasks, Targets);
}

bool LocalizationCommandletTasks::ImportDialogueForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString ImportDialogueScriptPath = LocalizationConfigurationScript::GetImportDialogueConfigPath(Target);
	LocalizationConfigurationScript::GenerateImportDialogueConfigFile(Target).WriteWithSCC(ImportDialogueScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ImportDialogueTaskName", "Import Dialogue"), ImportDialogueScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ImportDialogueForTargetWindowTitle", "Import Dialogue for Target {TargetName}"), Arguments);
	return ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
}

bool LocalizationCommandletTasks::ImportDialogueForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName)
{
	FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
	if (!Culture.IsValid())
	{
		return false;
	}

	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString DefaultImportDialogueScriptPath = LocalizationConfigurationScript::GetImportDialogueConfigPath(Target, TOptional<FString>(CultureName));
	const FString ImportDialogueScriptPath = FPaths::CreateTempFilename(*FPaths::GetPath(DefaultImportDialogueScriptPath), *FPaths::GetBaseFilename(DefaultImportDialogueScriptPath), *FPaths::GetExtension(DefaultImportDialogueScriptPath, true));
	LocalizationConfigurationScript::GenerateImportDialogueConfigFile(Target, TOptional<FString>(CultureName)).Write(ImportDialogueScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("ImportDialogueTaskName", "Import Dialogue"), ImportDialogueScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("ImportCultureDialogueForTargetWindowTitle", "Import {CultureName} Dialogue for Target {TargetName}"), Arguments);

	bool HasSucceeeded = ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
	IFileManager::Get().Delete(*ImportDialogueScriptPath); // Don't clutter up the loc config directory with scripts for individual cultures.
	return HasSucceeeded;
}

bool LocalizationCommandletTasks::GenerateWordCountReportsForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;

	for (ULocalizationTarget* Target : Targets)
	{
		const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

		const FText ReportTaskName = FText::Format(LOCTEXT("WordCountReportTaskNameFormat", "Generate Word Count Report for {TargetName}"), Arguments);
		const FString ReportScriptPath = LocalizationConfigurationScript::GetWordCountReportConfigPath(Target);
		LocalizationConfigurationScript::GenerateWordCountReportConfigFile(Target).WriteWithSCC(ReportScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(ReportTaskName, ReportScriptPath, ShouldUseProjectFile));
	}

	return ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(ParentWindow, LOCTEXT("GenerateReportsForAllTargetsWindowTitle", "Generate Word Count Reports for All Targets"), Tasks, Targets);
}

bool LocalizationCommandletTasks::GenerateWordCountReportForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString ReportScriptPath = LocalizationConfigurationScript::GetWordCountReportConfigPath(Target);
	LocalizationConfigurationScript::GenerateWordCountReportConfigFile(Target).WriteWithSCC(ReportScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("WordCountReportTaskName_NoTarget", "Generate Word Count Report"), ReportScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("GenerateReportForTargetWindowTitle", "Generate Word Count Report for Target {TargetName}"), Arguments);
	return ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
}

bool LocalizationCommandletTasks::CompileTextForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;

	for (ULocalizationTarget* Target : Targets)
	{
		const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

		const FText CompileTaskName = FText::Format(LOCTEXT("CompileTaskNameFormat", "Compile Translations for {TargetName}"), Arguments);
		const FString CompileScriptPath = LocalizationConfigurationScript::GetCompileTextConfigPath(Target);
		LocalizationConfigurationScript::GenerateCompileTextConfigFile(Target).WriteWithSCC(CompileScriptPath);
		Tasks.Add(LocalizationCommandletExecution::FTask(CompileTaskName, CompileScriptPath, ShouldUseProjectFile));
	}

	return ExecuteTasksAndBroadcastLocalizationTargetsDataUpdated(ParentWindow, LOCTEXT("GenerateLocResForAllTargetsWindowTitle", "Compile Translations for All Targets"), Tasks, Targets);
}

bool LocalizationCommandletTasks::CompileTextForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target)
{
	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString CompileScriptPath = LocalizationConfigurationScript::GetCompileTextConfigPath(Target);
	LocalizationConfigurationScript::GenerateCompileTextConfigFile(Target).WriteWithSCC(CompileScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("CompileTaskName", "Compile Translations"), CompileScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("GenerateLocResForTargetWindowTitle", "Compile Translations for Target {TargetName}"), Arguments);
	return ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
}

bool LocalizationCommandletTasks::CompileTextForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName)
{
	FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
	if (!Culture.IsValid())
	{
		return false;
	}

	TArray<LocalizationCommandletExecution::FTask> Tasks;
	const bool ShouldUseProjectFile = !Target->IsMemberOfEngineTargetSet();

	const FString DefaultCompileScriptPath = LocalizationConfigurationScript::GetCompileTextConfigPath(Target, TOptional<FString>(CultureName));
	const FString CompileScriptPath = FPaths::CreateTempFilename(*FPaths::GetPath(DefaultCompileScriptPath), *FPaths::GetBaseFilename(DefaultCompileScriptPath), *FPaths::GetExtension(DefaultCompileScriptPath, true));
	LocalizationConfigurationScript::GenerateCompileTextConfigFile(Target, TOptional<FString>(CultureName)).Write(CompileScriptPath);
	Tasks.Add(LocalizationCommandletExecution::FTask(LOCTEXT("CompileTaskName", "Compile Translations"), CompileScriptPath, ShouldUseProjectFile));

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("CultureName"), FText::FromString(Culture->GetDisplayName()));
	Arguments.Add(TEXT("TargetName"), FText::FromString(Target->Settings.Name));

	const FText WindowTitle = FText::Format(LOCTEXT("CompileCultureForTargetWindowTitle", "Compile {CultureName} Translations for Target {TargetName}"), Arguments);

	bool HasSucceeeded = ExecuteTasksAndBroadcastLocalizationTargetDataUpdated(ParentWindow, WindowTitle, Tasks, Target);
	IFileManager::Get().Delete(*CompileScriptPath); // Don't clutter up the loc config directory with scripts for individual cultures.
	return HasSucceeeded;
}

bool LocalizationCommandletTasks::ReportLoadedAudioAssets(const TArray<ULocalizationTarget*>& Targets, const TOptional<FString>& CultureName)
{
	TSet<FString> LoadedDialogueWaveAssets;
	TSet<FString> LoadedSoundWaveAssets;

	for (const ULocalizationTarget* Target : Targets)
	{
		const FString RootAssetPath = Target->IsMemberOfEngineTargetSet() ? TEXT("/Engine") : TEXT("/Game");

		TArray<FString> CulturesToTest;
		{
			if (CultureName.IsSet())
			{
				CulturesToTest.Add(CultureName.GetValue());
			}
			else
			{
				CulturesToTest.Reserve(Target->Settings.SupportedCulturesStatistics.Num());
				for (const FCultureStatistics& CultureData : Target->Settings.SupportedCulturesStatistics)
				{
					CulturesToTest.Add(CultureData.CultureName);
				}
			}
		}

		TArray<FString> DialogueWavePathsToTest;
		TArray<FString> SoundWavePathsToTest;
		{
			const FString NativeCulture = Target->Settings.SupportedCulturesStatistics.IsValidIndex(Target->Settings.NativeCultureIndex) ? Target->Settings.SupportedCulturesStatistics[Target->Settings.NativeCultureIndex].CultureName : FString();
			const bool bImportNativeAsSource = Target->Settings.ImportDialogueSettings.bImportNativeAsSource && !NativeCulture.IsEmpty();
			if (bImportNativeAsSource)
			{
				DialogueWavePathsToTest.Add(RootAssetPath);
				SoundWavePathsToTest.Add(RootAssetPath / Target->Settings.ImportDialogueSettings.ImportedDialogueFolder);
			}

			for (const FString& Culture : CulturesToTest)
			{
				if (bImportNativeAsSource && Culture == NativeCulture)
				{
					continue;
				}

				// Need to check for both UE and Verse style identifiers, as both may be in use
				DialogueWavePathsToTest.Add(RootAssetPath / TEXT("L10N") / Culture);
				SoundWavePathsToTest.Add(RootAssetPath / TEXT("L10N") / Culture / Target->Settings.ImportDialogueSettings.ImportedDialogueFolder);
				if (const FString VerseIdentifier = FCulture::CultureNameToVerseIdentifier(Culture);
					VerseIdentifier != Culture)
				{
					DialogueWavePathsToTest.Add(RootAssetPath / TEXT("L10N") / VerseIdentifier);
					SoundWavePathsToTest.Add(RootAssetPath / TEXT("L10N") / VerseIdentifier / Target->Settings.ImportDialogueSettings.ImportedDialogueFolder);
				}
			}
		}

		ForEachObjectOfClass(UDialogueWave::StaticClass(), [&](UObject* InObject)
		{
			const FString ObjectPath = InObject->GetPathName();

			auto FindAssetPathPredicate = [&](const FString& InAssetPath) -> bool
			{
				return ObjectPath.StartsWith(InAssetPath, ESearchCase::IgnoreCase);
			};

			if (DialogueWavePathsToTest.ContainsByPredicate(FindAssetPathPredicate))
			{
				LoadedDialogueWaveAssets.Add(ObjectPath);
			}
		});

		ForEachObjectOfClass(USoundWave::StaticClass(), [&](UObject* InObject)
		{
			const FString ObjectPath = InObject->GetPathName();

			auto FindAssetPathPredicate = [&](const FString& InAssetPath) -> bool
			{
				return ObjectPath.StartsWith(InAssetPath, ESearchCase::IgnoreCase);
			};

			if (SoundWavePathsToTest.ContainsByPredicate(FindAssetPathPredicate))
			{
				LoadedSoundWaveAssets.Add(ObjectPath);
			}
		});
	}

	if (LoadedDialogueWaveAssets.Num() > 0 || LoadedSoundWaveAssets.Num() > 0)
	{
		FTextBuilder MsgBuilder;
		MsgBuilder.AppendLine(LOCTEXT("Warning_LoadedAudioAssetsMsg", "The following audio assets have been loaded by the editor and may cause the dialogue import to fail as their files will be read-only."));
		MsgBuilder.AppendLine(FText::GetEmpty());
		MsgBuilder.AppendLine(LOCTEXT("Warning_LoadedAudioAssetsMsg_Continue", "Do you want to continue?"));
				
		if (LoadedDialogueWaveAssets.Num() > 0)
		{
			MsgBuilder.AppendLine(FText::GetEmpty());
			MsgBuilder.AppendLine(LOCTEXT("Warning_LoadedAudioAssetsMsg_DialogueWaves", "Dialogue Waves:"));

			MsgBuilder.Indent();
			for (const FString& LoadedDialogueWaveAsset : LoadedDialogueWaveAssets)
			{
				MsgBuilder.AppendLine(LoadedDialogueWaveAsset);
			}
			MsgBuilder.Unindent();
		}

		if (LoadedSoundWaveAssets.Num() > 0)
		{
			MsgBuilder.AppendLine(FText::GetEmpty());
			MsgBuilder.AppendLine(LOCTEXT("Warning_LoadedAudioAssetsMsg_SoundWaves", "Sound Waves:"));

			MsgBuilder.Indent();
			for (const FString& LoadedSoundWaveAsset : LoadedSoundWaveAssets)
			{
				MsgBuilder.AppendLine(LoadedSoundWaveAsset);
			}
			MsgBuilder.Unindent();
		}

		return FMessageDialog::Open(EAppMsgType::YesNo, MsgBuilder.ToText(), LOCTEXT("Warning_LoadedAudioAssetsTitle", "Warning - Loaded Audio Assets")) == EAppReturnType::Yes;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
