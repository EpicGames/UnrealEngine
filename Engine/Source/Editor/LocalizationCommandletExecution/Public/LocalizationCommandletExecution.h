// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"

#define UE_API LOCALIZATIONCOMMANDLETEXECUTION_API

class SWindow;

namespace LocalizationCommandletExecution
{
	struct FTask
	{
		FTask() {}

		FTask(const FText& InName, const FString& InScriptPath, const bool InShouldUseProjectFile)
			: Name(InName)
			, ScriptPath(InScriptPath)
			, ShouldUseProjectFile(InShouldUseProjectFile)
		{}

		FText Name;
		FString ScriptPath;
		bool ShouldUseProjectFile;
	};

	bool Execute(const TSharedRef<SWindow>& ParentWindow, const FText& Title, const TArray<FTask>& Tasks);
};

class FLocalizationCommandletProcess : public TSharedFromThis<FLocalizationCommandletProcess>
{
public:
	static UE_API TSharedPtr<FLocalizationCommandletProcess> Execute(const FString& ConfigFilePath, const bool UseProjectFile = true);

private:
	FLocalizationCommandletProcess(void* const InReadPipe, void* const InWritePipe, const FProcHandle InProcessHandle, const FString& InProcessArguments)
		: ReadPipe(InReadPipe)
		, WritePipe(InWritePipe)
		, ProcessHandle(InProcessHandle)
		, ProcessArguments(InProcessArguments)
	{
	}

public:
	UE_API ~FLocalizationCommandletProcess();

	void* GetReadPipe() const
	{
		return ReadPipe;
	}

	FProcHandle& GetHandle()
	{
		return ProcessHandle;
	}

	const FString& GetProcessArguments() const
	{
		return ProcessArguments;
	}

private:
	void* const ReadPipe;
	void* const WritePipe;
	FProcHandle ProcessHandle;
	FString ProcessArguments;
};

#undef UE_API
