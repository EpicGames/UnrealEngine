// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class SWindow;
class ULocalizationTarget;

namespace LocalizationCommandletTasks
{
	LOCALIZATIONCOMMANDLETEXECUTION_API bool GatherTextForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets);
	LOCALIZATIONCOMMANDLETEXECUTION_API bool GatherTextForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target);

	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportTextForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets, const TOptional<FString> DirectoryPath = TOptional<FString>());
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportTextForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const TOptional<FString> DirectoryPath = TOptional<FString>());
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportTextForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName, const TOptional<FString> FilePath = TOptional<FString>());

	LOCALIZATIONCOMMANDLETEXECUTION_API bool ExportTextForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets, const TOptional<FString> DirectoryPath = TOptional<FString>());
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ExportTextForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const TOptional<FString> DirectoryPath = TOptional<FString>());
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ExportTextForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName, const TOptional<FString> FilePath = TOptional<FString>());

	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportDialogueScriptForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets, const TOptional<FString> DirectoryPath = TOptional<FString>());
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportDialogueScriptForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const TOptional<FString> DirectoryPath = TOptional<FString>());
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportDialogueScriptForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName, const TOptional<FString> FilePath = TOptional<FString>());

	LOCALIZATIONCOMMANDLETEXECUTION_API bool ExportDialogueScriptForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets, const TOptional<FString> DirectoryPath = TOptional<FString>());
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ExportDialogueScriptForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const TOptional<FString> DirectoryPath = TOptional<FString>());
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ExportDialogueScriptForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName, const TOptional<FString> FilePath = TOptional<FString>());

	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportDialogueForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets);
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportDialogueForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target);
	LOCALIZATIONCOMMANDLETEXECUTION_API bool ImportDialogueForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName);

	LOCALIZATIONCOMMANDLETEXECUTION_API bool GenerateWordCountReportsForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets);
	LOCALIZATIONCOMMANDLETEXECUTION_API bool GenerateWordCountReportForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target);

	LOCALIZATIONCOMMANDLETEXECUTION_API bool CompileTextForTargets(const TSharedRef<SWindow>& ParentWindow, const TArray<ULocalizationTarget*>& Targets);
	LOCALIZATIONCOMMANDLETEXECUTION_API bool CompileTextForTarget(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target);
	LOCALIZATIONCOMMANDLETEXECUTION_API bool CompileTextForCulture(const TSharedRef<SWindow>& ParentWindow, ULocalizationTarget* const Target, const FString& CultureName);

	LOCALIZATIONCOMMANDLETEXECUTION_API bool ReportLoadedAudioAssets(const TArray<ULocalizationTarget*>& Targets, const TOptional<FString>& CultureName = TOptional<FString>());
}
